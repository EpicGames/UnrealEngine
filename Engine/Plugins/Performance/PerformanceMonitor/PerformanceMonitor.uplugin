{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "PerformanceMonitor",
	"Description": "A plugin for tracking the value of certain timers during gameplay.",
	"Category" : "Performance",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "PerformanceMonitor",
			"Type" : "Runtime",
			"LoadingPhase" : "Default"
		}
	]
}