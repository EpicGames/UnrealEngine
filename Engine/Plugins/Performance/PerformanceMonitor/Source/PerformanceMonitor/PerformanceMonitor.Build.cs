// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class PerformanceMonitor : ModuleRules
    {
        public PerformanceMonitor(ReadOnlyTargetRules Target) : base(Target)
        {
            PublicDependencyModuleNames.AddRange
            (
                new string[] {
				"Core",
				"CoreUObject",
                "Engine",
                "InputCore",
				"RHI",
				"RenderCore",
			}
            );

            if (Target.bBuildEditor)
            {
                PrivateDependencyModuleNames.AddRange(
                    new string[] {
					
				}
                );
            }

        }
    }
}
