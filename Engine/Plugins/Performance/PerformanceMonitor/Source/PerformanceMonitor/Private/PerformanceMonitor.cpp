// Copyright Epic Games, Inc. All Rights Reserved.

//#include "OrionGame.h"
#include "PerformanceMonitor.h"
#include "HAL/FileManager.h"

#include "Engine/Engine.h"
#include "EngineGlobals.h"
#include "Engine/GameViewportClient.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/AutomationTest.h"
#include "RenderTimer.h"
#include "Stats/StatsData.h"
#include "UnrealClient.h"
#include "DynamicRHI.h"

#define SUPER_DETAILED_AUTOMATION_STATS 1
PERFORMANCEMONITOR_API int ExportedInt;
IMPLEMENT_MODULE(FPerformanceMonitorModule, PerformanceMonitor);
FPerformanceMonitorModule::FPerformanceMonitorModule()
{
	bRecording = false;
	FileToLogTo = nullptr;
	for (int i = 0; i < 10; i++)
	{
		bNewFrameDataReady[i] = false;
	}
	TimeBetweenRecords = 0.01f;
	TestTimeOut = 0.f;
	NumOfCapturedFrames = 0;
	NumOfFramesToCapture = 0;
	bCvsToolsMode = false;
}

FPerformanceMonitorModule::~FPerformanceMonitorModule()
{
	if (FileToLogTo)
	{
		FileToLogTo->Close();
	}
}

bool FPerformanceMonitorModule::Tick(float DeltaTime)
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FPerformanceMonitorModule_Tick);
#if STATS

	if (bRecording)
	{
		if (FPlatformTime::Seconds() - TimeOfLastRecord > TimeBetweenRecords)
		{
			RecordFrame();
			TimeOfLastRecord = FPlatformTime::Seconds();
		}
	}
#endif
	return true;
}

void FPerformanceMonitorModule::StartupModule()
{

}

void FPerformanceMonitorModule::ShutdownModule()
{

}

void FPerformanceMonitorModule::Init()
{
	
}

void FPerformanceMonitorModule::GetDataFromStatsThread(int64 CurrentFrame)
{
#if STATS
	// We store 10 frames in the buffer at any given time.
	for (int i = 0; i < 10; i++)
	{
		if (!bNewFrameDataReady[i] && CurrentFrame >= 0)
		{
			FStatsThreadState& Stats = FStatsThreadState::GetLocalState();
			ReceivedFramePayload[i] = Stats.GetCondensedHistory(CurrentFrame);
			bNewFrameDataReady[i] = true;
			return;
		}
	}
#endif
}

void FPerformanceMonitorModule::SetRecordInterval(float NewInterval)
{
	TimeBetweenRecords = NewInterval;
}

bool FPerformanceMonitorModule::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
#if STATS
	// Ignore any execs that don't start with Lobby
	if (FParse::Command(&Cmd, TEXT("PerformanceMonitor")))
	{
		if (FParse::Command(&Cmd, TEXT("start")))
		{
			if (!bRecording)
			{
				FString StringCommand = Cmd;
				if (!StringCommand.IsEmpty())
				{
					StartRecordingPerfTimers(StringCommand, DesiredStats);
					TickHandler = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FPerformanceMonitorModule::Tick));
				}
			}
			else
			{
				Ar.Logf(TEXT("PerformanceMonitor is already running!"));
			}
		}
		else if (FParse::Command(&Cmd, TEXT("stop")))
		{
			if (bRecording)
			{
				FTSTicker::RemoveTicker(TickHandler);
				StopRecordingPerformanceTimers();
			}
			else
			{
				Ar.Logf(TEXT("PerformanceMonitor can't stop because it isn't running!"));
			}

		}
		else if (FParse::Command(&Cmd, TEXT("addtimer")))
		{
			
			FString StringCommand = Cmd;
			if (!StringCommand.IsEmpty())
			{
				DesiredStats.AddUnique(StringCommand);
			}
		}
		else if (FParse::Command(&Cmd, TEXT("setinterval")))
		{
			FString StringCommand = Cmd;
			if (!StringCommand.IsEmpty())
			{
				float NewTime = FCString::Atof(*StringCommand);
				if (NewTime)
				{
					TimeBetweenRecords = NewTime;
				}
			}
		}
		else if (FParse::Command(&Cmd, TEXT("cvstoolsmode")))
		{
			FString StringCommand = Cmd;
			if (!StringCommand.IsEmpty())
			{
				bCvsToolsMode = FCString::ToBool(*StringCommand);
			}
		}
		else
		{
			Ar.Logf(TEXT("Incorrect PerformanceMonitor command syntax! Supported commands are: "));
			Ar.Logf(TEXT("\tPerformanceMonitor start <filename as string>"));
			Ar.Logf(TEXT("\tPerformanceMonitor stop"));
			Ar.Logf(TEXT("\tPerformanceMonitor setinterval <seconds as float>"));
			Ar.Logf(TEXT("\tPerformanceMonitor addtimer <timername as string>"));
			Ar.Logf(TEXT("\tPerformanceMonitor cvstoolsmode <true|false>"));
		}


		return true;
	}
#endif
	return false;
}



void FPerformanceMonitorModule::StartRecordingPerfTimers(FString FileNameToUse, TArray<FString> StatsToRecord)
{
#if STATS

	for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
	{
		if (WorldContext.WorldType == EWorldType::Game || WorldContext.WorldType == EWorldType::PIE)
		{
			UWorld* World = WorldContext.World();
			UGameViewportClient* ViewportClient = World->GetGameViewport();
			
			GEngine->SetEngineStat(World, ViewportClient, TEXT("Unit"), true);
			GEngine->SetEngineStat(World, ViewportClient, TEXT("Particles"), true);
			GEngine->SetEngineStat(World, ViewportClient, TEXT("Anim"), true);
			GEngine->SetEngineStat(World, ViewportClient, TEXT("GpuParticles"), true);
			break;
		}
	}
	if (bRecording)
	{
		GLog->Log(TEXT("PerformanceMonitor"), ELogVerbosity::Warning, TEXT("Tried to start recording when we already have started! Don't do that."));
		return;
	}

	if (FileNameToUse == TEXT(""))	
	{
		GLog->Log(TEXT("PerformanceMonitor"), ELogVerbosity::Warning, TEXT("Please set a file name."));
		LogFileName = TEXT("UnnamedPerfData");
	}
	// Make sure we're not holding over any 
	ReceivedFramePayload->Empty();

	if (LogFileName != FileNameToUse)
	{
		if (FileToLogTo)
		{
			FileToLogTo->Close();
		}
		LogFileName = FileNameToUse;

		FString NewLogFileName = FString::Printf(TEXT("%sFXPerformance/%s.csv"), *FPaths::ProjectSavedDir(), *LogFileName);
		FileToLogTo = IFileManager::Get().CreateFileWriter(*NewLogFileName, false);
	}

	if (!FileToLogTo)
	{
		FString FileName = FString::Printf(TEXT("%s.csv"), *FileNameToUse);
		FileToLogTo =  IFileManager::Get().CreateFileWriter(*FileName, false);
	}
	if (GConfig)
	{
		FString ConfigCategory = FString::Printf(TEXT("/Plugins/PerformanceMonitor/%s"), *FileNameToUse);
		StatGroupsToUse.Empty();
		float floatValueReceived = 0;
		if (GConfig->GetFloat(*ConfigCategory,
			TEXT("PerformanceMonitorInterval"),
			floatValueReceived,
			GGameIni
			))
		{
			TimeBetweenRecords = floatValueReceived;
		}
		floatValueReceived = 0;
		if (GConfig->GetFloat(*ConfigCategory,
			TEXT("PerformanceMonitorTimeout"),
			floatValueReceived,
			GGameIni
			))
		{
			TestTimeOut = floatValueReceived;
		}
		int32 intValueReceived = 0;
		if (GConfig->GetInt(*ConfigCategory,
			TEXT("PerformanceMonitorNumOfFramesToCapture"),
			intValueReceived,
			GGameIni
		))
		{
			NumOfFramesToCapture = intValueReceived;
		}
		TArray<FString> TimersOfInterest;
		if (GConfig->GetArray(*ConfigCategory,
			TEXT("PerformanceMonitorTimers"), TimersOfInterest, GGameIni))
		{
			DesiredStats = TimersOfInterest;
		}
		TArray<FString> TimerGroupsOfInterest;
		if (GConfig->GetArray(*ConfigCategory,
			TEXT("PerformanceMonitorStatGroups"), TimerGroupsOfInterest, GGameIni))
		{
			StatGroupsToUse = TimerGroupsOfInterest;
		}
		FString MapToLoad;
		if (GConfig->GetString(*ConfigCategory,
			TEXT("PerformanceMonitorMap"), MapToLoad, GGameIni))
		{
			MapToTest = MapToLoad;
		}
		bExitOnCompletion = false;
		bool GatheredBool;
		if (GConfig->GetBool(*ConfigCategory,
			TEXT("PerformanceMonitorExitOnFinish"), GatheredBool, GGameIni))
		{
			bExitOnCompletion = GatheredBool;
		}
		bRequiresCutsceneStart = false;
		GatheredBool = false;
		if (GConfig->GetBool(*ConfigCategory,
			TEXT("PerformanceMonitorRequireCutsceneStart"), GatheredBool, GGameIni))
		{
			bRequiresCutsceneStart = GatheredBool;
		}
	}
	FThreadStats::PrimaryEnableAdd(1);
	FAutomationTestFramework::Get().SetCaptureStack(false);
	FStatsThreadState& Stats = FStatsThreadState::GetLocalState();
	// Set up our delegate to gather data from the stats thread for safe consumption on game thread.
	Stats.NewFrameDelegate.AddRaw(this, &FPerformanceMonitorModule::GetDataFromStatsThread);
	// Cut down the flow of stats if we can to make things work more efficiently.
	if (StatGroupsToUse.Num())
	{
		for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
		{
			if (WorldContext.WorldType == EWorldType::Game || WorldContext.WorldType == EWorldType::PIE)
			{
				UWorld* World = WorldContext.World();
				GEngine->Exec(World, TEXT("stat group none"));
				for (int i = 0; i < StatGroupsToUse.Num(); i++)
				{
					FString StatGroupCommand = FString::Printf(TEXT("stat group enable %s"), *StatGroupsToUse[i]);
					GEngine->Exec(World, *StatGroupCommand);
				}
			}
		}
	}
	GeneratedStats.Empty();
	for (int i = 0; i < DesiredStats.Num(); i++)
	{
		TArray<float> TempArray;
		GeneratedStats.Add(DesiredStats[i], TempArray);
	}
	bRecording= true;
	TimeOfTestStart = FPlatformTime::Seconds();
	if (bRequiresCutsceneStart)
	{
		for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
		{
			if (WorldContext.WorldType == EWorldType::Game || WorldContext.WorldType == EWorldType::PIE)
			{
				UWorld* World = WorldContext.World();
				GEngine->Exec(World, TEXT("ce start"));
			}
		}
	}
#endif
}


void FPerformanceMonitorModule::RecordFrame()
{
	if (!bRecording)
	{
		return;
	}
	for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
	{
		if (WorldContext.WorldType == EWorldType::Game || WorldContext.WorldType == EWorldType::PIE)
		{
			UWorld* World = WorldContext.World();
			const FStatUnitData* StatUnitData = World->GetGameViewport()->GetStatUnitData();
			check(StatUnitData);
			TArray<float> ArrayForStatName = GeneratedStats.FindOrAdd(TEXT("FrameTime"));
			ArrayForStatName.Add(StatUnitData->RawFrameTime);
			GeneratedStats.Emplace(TEXT("FrameTime"), ArrayForStatName);
			ArrayForStatName = GeneratedStats.FindOrAdd(TEXT("RenderThreadTime"));
			ArrayForStatName.Add(StatUnitData->RawRenderThreadTime);
			GeneratedStats.Emplace(TEXT("RenderThreadTime"), ArrayForStatName);
			ArrayForStatName = GeneratedStats.FindOrAdd(TEXT("GameThreadTime"));
			ArrayForStatName.Add(StatUnitData->RawGameThreadTime);
			GeneratedStats.Emplace(TEXT("GameThreadTime"), ArrayForStatName);
			ArrayForStatName = GeneratedStats.FindOrAdd(TEXT("GPUFrameTime"));
			ArrayForStatName.Add(StatUnitData->RawGPUFrameTime[0]);
			GeneratedStats.Emplace(TEXT("GPUFrameTime"), ArrayForStatName);

			ArrayForStatName = GeneratedStats.FindOrAdd(TEXT("GlobalRenderThreadTime"));
			ArrayForStatName.Add(FGenericPlatformTime::ToMilliseconds(GRenderThreadTime));
			GeneratedStats.Emplace(TEXT("GlobalRenderThreadTime"), ArrayForStatName);
			ArrayForStatName = GeneratedStats.FindOrAdd(TEXT("GlobalGameThreadTime"));
			ArrayForStatName.Add(FGenericPlatformTime::ToMilliseconds(GGameThreadTime));
			GeneratedStats.Emplace(TEXT("GlobalGameThreadTime"), ArrayForStatName);
			ArrayForStatName = GeneratedStats.FindOrAdd(TEXT("GlobalGPUFrameTime"));
			ArrayForStatName.Add(FGenericPlatformTime::ToMilliseconds(RHIGetGPUFrameCycles()));
			GeneratedStats.Emplace(TEXT("GlobalGPUFrameTime"), ArrayForStatName);

			break;
		}
	}
	//GetStatsBreakdown();
	if (TestTimeOut && (FPlatformTime::Seconds() - TestTimeOut > TimeOfTestStart))
	{
		StopRecordingPerformanceTimers();
	}

	if (NumOfFramesToCapture > 0)
	{
		++NumOfCapturedFrames;

		if (NumOfCapturedFrames >= NumOfFramesToCapture)
		{
			StopRecordingPerformanceTimers();
		}
	}
}

void FPerformanceMonitorModule::GetStatsBreakdown()
{
#if STATS		
		TArray<FString> StatsCoveredThisFrame = DesiredStats;
		for (int i = 0; i < 10; i++)
		{
			if (bNewFrameDataReady[i])
			{
				for (int j = 0; j < ReceivedFramePayload[i].Num(); j++)
				{
					TArray<float> ArrayForStatName;
					FStatMessage TempMessage = ReceivedFramePayload[i][j];


					FName StatFName = TempMessage.NameAndInfo.GetShortName();
					if (!StatFName.IsValid())
					{
						return;
					}

					FString StatName = StatFName.ToString();
					if (StatsCoveredThisFrame.Contains(StatName))
					{
						ArrayForStatName = GeneratedStats.FindOrAdd(StatName);
						ArrayForStatName.Add(FPlatformTime::ToMilliseconds(TempMessage.GetValue_Duration()));
						GeneratedStats.Emplace(StatName, ArrayForStatName);
						StatsCoveredThisFrame.Remove(StatName);
					}
					if (!StatsCoveredThisFrame.Num())
					{
						continue;
					}
				}
				ReceivedFramePayload[i].Empty();
				bNewFrameDataReady[i] = false;
			}
		}
#endif
}


void FPerformanceMonitorModule::FinalizeFTestPerfReport()
{
#if STATS
	if (bRecording)
	{
		StopRecordingPerformanceTimers();
	}
	if (FileToLogTo)
	{
		FileToLogTo->Close();
	}
	LogFileName = TEXT("");
#endif
}

void FPerformanceMonitorModule::StopRecordingPerformanceTimers()
{
#if STATS
	if (!bRecording)
	{
		GLog->Log(TEXT("FTestPerfRecorder"), ELogVerbosity::Warning, TEXT("Tried to End Recording when we haven't started recording! Don't do that."));
		return;
	}

	if (bCvsToolsMode)
	{
		RecordDataInCvsToolsMode();
	}
	else
	{
		RecordData();
	}

	FThreadStats::PrimaryEnableSubtract(1);
	FStatsThreadState& Stats = FStatsThreadState::GetLocalState();
	Stats.NewFrameDelegate.RemoveAll(this);
	bRecording = false;
	if (!GeneratedStats.Num())
	{
		bRecording = false;
		GLog->Log(TEXT("FTestPerfRecorder"), ELogVerbosity::Warning, TEXT("No perf data to record. "));
		return;
	}
	if (FileToLogTo)
	{
		FileToLogTo->Close();
	}
	FAutomationTestFramework::Get().SetCaptureStack(true);


	if (bExitOnCompletion)
	{
		FPlatformMisc::RequestExit(false);
	}
#endif
}

float FPerformanceMonitorModule::GetAverageOfArray(TArray<float>ArrayToAvg, FString StatName)
{
	float RetVal = 0.f;
	int NumValidValues = 0;
	int NumOutlierValues = 0;
	float AvgWithOutliers = 0.f;
	float MaxOutlierValue = 0.f;
	bool bSlowStart = true;
	for (int i = 0; i < ArrayToAvg.Num(); i++)
	{
		if (ArrayToAvg[i] >= 0)
		{
			AvgWithOutliers += ArrayToAvg[i];
			NumValidValues++;
		}
	}
	if (NumValidValues)
	{
		AvgWithOutliers /= NumValidValues;
	}

	NumValidValues = 0;
	for (int i = 0; i < ArrayToAvg.Num(); i++)
	{
		if (ArrayToAvg[i] >= 0)
		{
			// Anything more than twice the average (including outliers) is likely an outlier.
			// Count and filter out the ones that occur at the beginning.
			if (bSlowStart && (ArrayToAvg[i] > (AvgWithOutliers * 2)))
			{
				NumOutlierValues++;
				if (ArrayToAvg[i] > MaxOutlierValue)
				{
					MaxOutlierValue = ArrayToAvg[i];
				}
			}
			else
			{
				bSlowStart = false;
				RetVal += ArrayToAvg[i];
				NumValidValues++;
			}
		}
	}

	if (NumOutlierValues)
	{
		GLog->Log(TEXT("PerformanceMonitor"), ELogVerbosity::Warning, FString::Printf(TEXT("Stat Array for %s contained %d initial outliers, the max of which was %0.4f"), *StatName, NumOutlierValues, MaxOutlierValue));
	}

	if (NumValidValues)
	{
		RetVal /= NumValidValues;
	}
	return RetVal;
}

void FPerformanceMonitorModule::RecordDataInCvsToolsMode()
{
#if STATS

	int MaxStats = 0;

	// Print header
	FString StringToPrint;
	for (TMap<FString, TArray<float>>::TConstIterator ThreadedItr(GeneratedStats); ThreadedItr; ++ThreadedItr)
	{
		if (!StringToPrint.IsEmpty())
		{
			StringToPrint += TEXT(",");
		}

		StringToPrint += ThreadedItr.Key();

		MaxStats = FMath::Max(MaxStats, ThreadedItr.Value().Num());
	}

	StringToPrint += TEXT("\n");
	FileToLogTo->Serialize(TCHAR_TO_ANSI(*StringToPrint), StringToPrint.Len());

	for (int i = 0; i < MaxStats; ++i)
	{
		StringToPrint.Empty();

		for (TMap<FString, TArray<float>>::TConstIterator ThreadedItr(GeneratedStats); ThreadedItr; ++ThreadedItr)
		{
			float StatValue = -1;
			const TArray<float>& StatValues = ThreadedItr.Value();
			if (i < StatValues.Num())
			{
				StatValue = StatValues[i];
			}

			if (!StringToPrint.IsEmpty())
			{
				StringToPrint += TEXT(",");
			}

			StringToPrint += FString::Printf(TEXT("%0.4f"), StatValue);
		}

		StringToPrint += TEXT("\n");
		FileToLogTo->Serialize(TCHAR_TO_ANSI(*StringToPrint), StringToPrint.Len());
	}

#endif
}

void FPerformanceMonitorModule::RecordData()
{
#if STATS
	FString StringToPrint = FString::Printf(TEXT("Interval (s),%0.4f\n"), TimeBetweenRecords);
	FileToLogTo->Serialize(TCHAR_TO_ANSI(*StringToPrint), StringToPrint.Len());
	for (TMap<FString, TArray<float>>::TConstIterator ThreadedItr(GeneratedStats); ThreadedItr; ++ThreadedItr)
	{
				StringToPrint = ThreadedItr.Key();
				TArray<float> StatValues = ThreadedItr.Value();
				for (int i = 0; i < StatValues.Num(); i++)
				{
					StringToPrint = FString::Printf(TEXT("%s,%0.4f"), *StringToPrint, StatValues[i]);
				}
				StringToPrint.Append(TEXT("\n"));
				FileToLogTo->Serialize(TCHAR_TO_ANSI(*StringToPrint), StringToPrint.Len());
	}
	StringToPrint = TEXT("Timer Name, Min Val, Max Val, Avg Val, Timer Active Frames\n");
	FileToLogTo->Serialize(TCHAR_TO_ANSI(*StringToPrint), StringToPrint.Len());
	for (TMap<FString, TArray<float>>::TConstIterator ThreadedItr(GeneratedStats); ThreadedItr; ++ThreadedItr)
	{
		StringToPrint = ThreadedItr.Key();
		if (StringToPrint == TEXT("TimeSinceStart"))
		{
			continue;
		}
		TArray<float> StatValues = ThreadedItr.Value();
		float StatMin = FMath::Min(StatValues);
		float StatMax = FMath::Max(StatValues);
		float StatAvg = GetAverageOfArray(StatValues, ThreadedItr.Key());
		int ActiveFrames = StatValues.Num();
		// Skip 0 frame because it tends to be the hitchiest frame.
		for (int i = 1; i < StatValues.Num(); i++)
		{
			if (!StatValues[i])
			{
				ActiveFrames--;
			}
		}
		StringToPrint = FString::Printf(TEXT("%s,%0.4f,%0.4f,%0.4f,%d\n"), *StringToPrint, StatMin, StatMax, StatAvg, ActiveFrames);
		FileToLogTo->Serialize(TCHAR_TO_ANSI(*StringToPrint), StringToPrint.Len());
	}
#endif
}

void FPerformanceMonitorModule::CleanUpPerfFileHandles()
{
#if STATS

	if (FileToLogTo)
	{
		FileToLogTo->Close();
	}
	LogFileName.Empty();
	DesiredStats.Empty();
#endif
}
