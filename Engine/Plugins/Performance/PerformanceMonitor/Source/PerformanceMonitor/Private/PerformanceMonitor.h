// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// #include "Tests/OrionAutomationHelper.h"
#include "Modules/ModuleManager.h"
#include "Containers/Ticker.h"

struct FStatMessage;

class FPerformanceMonitorModule : public IModuleInterface, public FSelfRegisteringExec
{
	//Mapping between thread name and stats, where stats are a mapping between name and time
	struct FStatData
	{
		float StatIncAvg;
		float StatIncMax;
		int FrameNum;
	};
private: 
	TArray<FString> DesiredStats;
	TArray<FString> StatGroupsToUse;
#if STATS
	TArray<TArray<FStatMessage>> StoredMessages;
	// Received frame data. Cleared after parsing.
	TArray<FStatMessage>ReceivedFramePayload[10];
#endif
	TMap<FString, TArray<float>> GeneratedStats;
	FArchive * FileToLogTo;
	FString LogFileName;
	FTSTicker::FDelegateHandle TickHandler;

	bool bRecording;
	float TimeOfTestStart;
	float TestTimeOut;
	float TimeOfLastRecord;
	float TimeBetweenRecords;
	int32 NumOfCapturedFrames;
	int32 NumOfFramesToCapture;
	FString MapToTest;
	bool bHasWarnedAboutTime;
	bool bExitOnCompletion;
	bool bRequiresCutsceneStart;
	bool bCvsToolsMode;
protected:
public:

	FPerformanceMonitorModule();
	~FPerformanceMonitorModule();
	// FSelfRegisteringExec
	virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
	
	virtual void StartupModule() override;

	virtual void ShutdownModule() override;

	void Init();

	/**
	* Singleton-like access to this module's interface.  This is just for convenience!
	* Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	*
	* @return Returns singleton instance, loading the module on demand if needed
	*/
	static inline FPerformanceMonitorModule& Get()
	{
		return FModuleManager::LoadModuleChecked<FPerformanceMonitorModule>("PerformanceMonitor");
	}

	/**
	* Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	*
	* @return True if the module is loaded and ready to use
	*/
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("PerformanceMonitor");
	}

	///** [FTickableGameObject] tick function */
	bool Tick(float DeltaTime);

	///** [FTickableGameObject] tick when recording as long as we are not CDO. */
	bool IsTickable() const { return bRecording; }

	// Whether the received frame is ready to be interacted with on the game thread.
	bool bNewFrameDataReady[10];
	void GetDataFromStatsThread(int64 CurrentFrame);

	bool IsRecordingPerfTimers() { return bRecording; }
	void StartRecordingPerfTimers(FString FileName, TArray<FString> StatsToRecord);
	void SetRecordInterval(float NewInterval);

	void RecordFrame();
	void GetStatsBreakdown();
	void RecordDataInCvsToolsMode();
	void RecordData();	

	float GetAverageOfArray(TArray<float>ArrayToAvg, FString StatName);
	
	void StopRecordingPerformanceTimers();
	void FinalizeFTestPerfReport();
	void CleanUpPerfFileHandles();

};
 
